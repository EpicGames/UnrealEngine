{
	"FileVersion": 1,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "MVVM Toolset",
	"Description": "Toolset for creating and managing MVVM data via the AI Toolset Registry.",
	"Category": "Other",
	"EditorOnly": true,
	"IsExperimentalVersion": true,
	"NoRedist": true,
	"EnabledByDefault": false,
	"Modules": [
		{
			"Name": "MVVMToolset",
			"Type": "Editor",
			"LoadingPhase": "PostEngineInit",
			"TargetAllowList": ["Editor"]
		}
	],
	"Plugins": [
		{
			"Name": "ToolsetRegistry",
			"Enabled": true
		},
		{
			"Name": "ModelViewViewModel",
			"Enabled": true
		}
	]
}
