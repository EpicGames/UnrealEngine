// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class MVVMToolset : ModuleRules
{
	public MVVMToolset(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.Add("Core");

		PrivateDependencyModuleNames.AddRange(new string[]
		{
			"AssetRegistry",
			"BlueprintEditorLibrary",
			"BlueprintGraph",
			"CoreUObject",
			"Engine",
			"Json",
			"Kismet",
			"ModelViewViewModel",
			"ModelViewViewModelBlueprint",
			"ModelViewViewModelEditor",
			"ToolsetRegistry",
			"UnrealEd",
			"UMG",
			"UMGEditor",
		});
	}
}
