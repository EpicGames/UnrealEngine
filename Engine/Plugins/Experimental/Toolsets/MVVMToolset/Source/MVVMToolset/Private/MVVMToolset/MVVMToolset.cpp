// Copyright Epic Games, Inc. All Rights Reserved.

#include "MVVMToolset.h"

#include "AssetRegistry/AssetRegistryHelpers.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "BlueprintEditorLibrary.h"
#include "Bindings/MVVMBindingHelper.h"
#include "Bindings/MVVMConversionFunctionHelper.h"
#include "Editor/EditorEngine.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraphSchema_K2.h"
#include "Engine/Blueprint.h"
#include "Factories/BlueprintFactory.h"
#include "Kismet/KismetSystemLibrary.h"
#include "Kismet2/Kismet2NameValidators.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Misc/PackageName.h"
#include "Misc/ScopeExit.h"
#include "MVVMBlueprintViewConversionFunction.h"
#include "MVVMEditorSubsystem.h"
#include "MVVMViewModelBase.h"
#include "MVVMWidgetBlueprintExtension_View.h"
#include "UObject/Package.h"
#include "UObject/SavePackage.h"
#include "UObject/UObjectGlobals.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MVVMToolset)

extern UNREALED_API UEditorEngine* GEditor;

namespace
{
	UMVVMBlueprintView* GetWidgetView(UWidgetBlueprint* WidgetBlueprint)
	{
		if (!IsValid(WidgetBlueprint))
		{
			UKismetSystemLibrary::RaiseScriptError(TEXT("Failed to get MVVM view for invalid widget blueprint."));
			return nullptr;
		}

		UMVVMEditorSubsystem* MVVMEditorSubsystem = GEditor->GetEditorSubsystem<UMVVMEditorSubsystem>();
		if (!IsValid(MVVMEditorSubsystem))
		{
			UKismetSystemLibrary::RaiseScriptError(FString::Printf(TEXT("Failed to get MVVM View for widget %s. UMVVMEditorSubsystem not loaded."), *WidgetBlueprint->GetName()));
			return nullptr;

		}

		UMVVMBlueprintView* View = MVVMEditorSubsystem->RequestView(WidgetBlueprint);
		if (!IsValid(View))
		{
			UKismetSystemLibrary::RaiseScriptError(FString::Printf(TEXT("Failed to get MVVM view for widget blueprint '%s'. UMVVMWidgetBlueprintExtension_View missing Blueprint View instance"), *WidgetBlueprint->GetName()));
			return nullptr;
		}

		return View;
	}

	FProperty* FindPropertyFromPath(UStruct* InStruct, const FString& Path)
	{
		TArray<FString> Segments;
		Path.ParseIntoArray(Segments, TEXT("."));

		UStruct* CurrentStruct = InStruct;
		FProperty* Found = nullptr;

		for (const FString& Segment : Segments)
		{
			if (!CurrentStruct)
			{
				return nullptr;
			}

			Found = CurrentStruct->FindPropertyByName(FName(*Segment));
			if (!Found)
			{
				return nullptr;
			}

			if (FStructProperty* StructProp = CastField<FStructProperty>(Found))
			{
				CurrentStruct = StructProp->Struct;
			}
			else if (FObjectProperty* ObjProp = CastField<FObjectProperty>(Found))
			{
				CurrentStruct = ObjProp->PropertyClass;
			}
			else
			{
				CurrentStruct = nullptr; // leaf — only valid if this is the last segment
			}
		}

		return Found;
	}

	FMVVMBlueprintPropertyPath GetPropertyPath(UWidgetBlueprint* WidgetBlueprint, UObject* Context, const FString& PropertyPath, FString& OutError)
	{
		if (WidgetBlueprint == nullptr)
		{
			OutError = FString::Printf(TEXT("Null WidgetBlueprint."));
			return FMVVMBlueprintPropertyPath();
		}

		if (Context == nullptr)
		{
			FProperty* Property = FindPropertyFromPath(WidgetBlueprint->GeneratedClass, PropertyPath);
			if (Property == nullptr)
			{
				OutError = FString::Printf(TEXT("Invalid property path '%s'. Property does not exist in '%s'"), *PropertyPath, *WidgetBlueprint->GetName());
				return FMVVMBlueprintPropertyPath();
			}

			// Get path to Widget Blueprint property
			FMVVMBlueprintPropertyPath BlueprintPropertyPath;
			BlueprintPropertyPath.SetSelfContext();
			BlueprintPropertyPath.AppendPropertyPath(WidgetBlueprint, UE::MVVM::FMVVMConstFieldVariant(Property));
			return BlueprintPropertyPath;
		}

		if (UWidget* Widget = Cast<UWidget>(Context))
		{
			FProperty* Property = FindPropertyFromPath(Widget->GetClass(), PropertyPath);
			if (Property == nullptr)
			{
				OutError = FString::Printf(TEXT("Invalid property path '%s'. Property does not exist in '%s'"), *PropertyPath, *Widget->GetName());
				return FMVVMBlueprintPropertyPath();
			}

			// Get path to Widget property
			FMVVMBlueprintPropertyPath BlueprintPropertyPath;
			BlueprintPropertyPath.SetWidgetName(Widget->GetFName());
			BlueprintPropertyPath.AppendPropertyPath(WidgetBlueprint, UE::MVVM::FMVVMConstFieldVariant(Property));
			return BlueprintPropertyPath;
		}

		if (UClass* ViewModelClass = Cast<UClass>(Context))
		{
			FProperty* Property = FindPropertyFromPath(ViewModelClass, PropertyPath);
			if (Property == nullptr)
			{
				OutError = FString::Printf(TEXT("Invalid property path '%s'. Property does not exist in '%s'"), *PropertyPath, *ViewModelClass->GetName());
				return FMVVMBlueprintPropertyPath();
			}

			// Get path to Widget View Model property
			UMVVMBlueprintView* View = GetWidgetView(WidgetBlueprint);
			if (!IsValid(View))
			{
				OutError = FString::Printf(TEXT("Invalid MVVM View."));
				return FMVVMBlueprintPropertyPath();
			}

			FGuid ViewModelId;
			for (const FMVVMBlueprintViewModelContext& ViewModelContext : View->GetViewModels())
			{
				if (Context == ViewModelContext.GetViewModelClass())
				{
					ViewModelId = ViewModelContext.GetViewModelId();
					break;
				}
			}

			if (!ViewModelId.IsValid())
			{
				OutError = FString::Printf(TEXT("ViewModel class '%s' not on widget blueprint %s."), *ViewModelClass->GetName(), *WidgetBlueprint->GetName());
				return FMVVMBlueprintPropertyPath();
			}

			FMVVMBlueprintPropertyPath BlueprintPropertyPath;
			BlueprintPropertyPath.SetViewModelId(ViewModelId);
			BlueprintPropertyPath.AppendPropertyPath(WidgetBlueprint, UE::MVVM::FMVVMConstFieldVariant(Property));
			return BlueprintPropertyPath;
		}


		OutError = FString::Printf(TEXT("Invalid Context '%s'. Must derive UWidget, UClass or be null."), *Context->GetName());
		return FMVVMBlueprintPropertyPath();
	}

	void TryAutoConvertBindingFromSourceToDestination(UWidgetBlueprint* WidgetBlueprint, FMVVMBlueprintViewBinding& Binding, FName ConversionName)
	{
		const FMVVMBlueprintPropertyPath SourcePath = Binding.SourcePath;
		const FMVVMBlueprintPropertyPath DestinationPath = Binding.DestinationPath;

		if (!SourcePath.IsValid() || !DestinationPath.IsValid())
		{
			return;
		}

		const FProperty* SourceProperty = nullptr;
		const FProperty* DestinationProperty = nullptr;

		const TArray<UE::MVVM::FMVVMConstFieldVariant> SourceFields = SourcePath.GetCompleteFields(WidgetBlueprint);
		if (SourceFields.Num() > 0 && SourceFields.Last().IsProperty())
		{
			SourceProperty = SourceFields.Last().GetProperty();
		}

		const TArray<UE::MVVM::FMVVMConstFieldVariant> DestinationFields = DestinationPath.GetCompleteFields(WidgetBlueprint);
		if (DestinationFields.Num() > 0 && DestinationFields.Last().IsProperty())
		{
			DestinationProperty = DestinationFields.Last().GetProperty();
		}

		if (SourceProperty == nullptr || DestinationProperty == nullptr)
		{
			// One binding is null, can not convert
			return;
		}

		if (UE::MVVM::BindingHelper::ArePropertiesCompatible(SourceProperty, DestinationProperty))
		{
			// if properties are compatible, no work to do
			return;
		}

		UMVVMEditorSubsystem* MVVMEditorSubsystem = GEditor->GetEditorSubsystem<UMVVMEditorSubsystem>();
		if (!IsValid(MVVMEditorSubsystem))
		{
			return;
		}

		UE::MVVM::FConversionFunctionValue SourceConversionFunction;
		FMVVMBlueprintPinId SourceArgumentPin;

		const TArray<UE::MVVM::FConversionFunctionValue> ConversionFunctions = MVVMEditorSubsystem->GetConversionFunctions(WidgetBlueprint, SourceProperty, DestinationProperty);
		if (!ConversionName.IsNone())
		{
			for (const UE::MVVM::FConversionFunctionValue& ConversionFunction : ConversionFunctions)
			{
				if (ConversionName == ConversionFunction.GetFName())
				{
					const TArray<FMVVMBlueprintPinId> ArgumentPins = MVVMEditorSubsystem->GetConversionFunctionPins(WidgetBlueprint, ConversionFunction, SourceProperty);
					if (!ArgumentPins.IsEmpty())
					{
						SourceArgumentPin = ArgumentPins[0];
					}
					SourceConversionFunction = ConversionFunction;
					break;
				}
			}
		}
		else
		{
			for (const UE::MVVM::FConversionFunctionValue& ConversionFunction : ConversionFunctions)
			{
				// Find conversions that have a source argument type and a destination return type
				const TArray<FMVVMBlueprintPinId> ArgumentPins = MVVMEditorSubsystem->GetConversionFunctionPins(WidgetBlueprint, ConversionFunction, SourceProperty);
				if (!ArgumentPins.IsEmpty())
				{
					SourceConversionFunction = ConversionFunction;
					SourceArgumentPin = ArgumentPins[0];
					break;
				}
			}
		}

		//Bind conversion function argument to the source path
		FMVVMBlueprintFunctionReference FunctionReference = SourceConversionFunction.IsFunction()
			? FMVVMBlueprintFunctionReference(WidgetBlueprint, SourceConversionFunction.GetFunction())
			: FMVVMBlueprintFunctionReference(SourceConversionFunction.GetNode());

		MVVMEditorSubsystem->SetSourceToDestinationConversionFunction(WidgetBlueprint, Binding, FunctionReference);
		
		if (SourceArgumentPin.IsValid())
		{
			MVVMEditorSubsystem->SetPathForConversionFunctionArgument(WidgetBlueprint, Binding, SourceArgumentPin, SourcePath, true);
		}
	}
}

UBlueprint* UMVVMToolset::CreateViewModel(const FString& ViewModelName, const FString& PackagePath, UClass* ParentClass)
{
	if (ParentClass == nullptr)
	{
		ParentClass = UMVVMViewModelBase::StaticClass();
	}

	if (!ParentClass  || !ParentClass->IsChildOf(UMVVMViewModelBase::StaticClass()))
	{
		UKismetSystemLibrary::RaiseScriptError(FString::Printf(TEXT("Failed to create ViewModel '%s'. ParentClassName '%s' does not refer to a UMVVMViewModelBase subclass."), *ViewModelName, ParentClass ? *ParentClass->GetName() : TEXT("null")));
		return nullptr;
	}

	const FString FullPackageName = PackagePath / ViewModelName;
	FText PackageNameError;
	if(!FPackageName::IsValidLongPackageName(FullPackageName, false, &PackageNameError))
	{
		UKismetSystemLibrary::RaiseScriptError(FString::Printf(TEXT("Failed to create ViewModel '%s'. Invalid package path '%s'. %s"), *ViewModelName, *FullPackageName, *(PackageNameError.ToString())));
		return nullptr;
	}

	if (FPackageName::DoesPackageExist(FullPackageName))
	{
		UKismetSystemLibrary::RaiseScriptError(FString::Printf(TEXT("Failed to create ViewModel '%s'. An asset already exists at '%s'."), *ViewModelName, *FullPackageName));
		return nullptr;
	}

	UPackage* Package = CreatePackage(*FullPackageName);
	if (!IsValid(Package))
	{
		UKismetSystemLibrary::RaiseScriptError(FString::Printf(TEXT("Failed to create ViewModel '%s'. Failed to create package at '%s'."), *ViewModelName, *FullPackageName));
		return nullptr;
	}

	UBlueprintFactory* Factory = NewObject<UBlueprintFactory>();
	Factory->ParentClass = ParentClass;
	UBlueprint* Blueprint = Cast<UBlueprint>(Factory->FactoryCreateNew(UBlueprint::StaticClass(), Package, FName(*ViewModelName), RF_Public | RF_Standalone, nullptr, GWarn));
	if (!IsValid(Blueprint))
	{
		UKismetSystemLibrary::RaiseScriptError(FString::Printf(TEXT("Failed to create ViewModel '%s'. Blueprint '%s' could not be created."), *ViewModelName, *ParentClass->GetName()));
		Package->MarkAsGarbage();
		CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);
		return nullptr;
	}

	FAssetRegistryModule::AssetCreated(Blueprint);
	return Blueprint;
}

bool UMVVMToolset::AddViewModelProperty(UBlueprint* ViewModel, const FString& PropertyName, const FString& PropertyType, const FString& DefaultValue)
{
	if (!IsValid(ViewModel))
	{
		UKismetSystemLibrary::RaiseScriptError(FString::Printf(TEXT("Failed to add ViewModel property '%s'. Invalid ViewModel."), *PropertyName));
		return false;
	}

	FKismetNameValidator Validator(ViewModel);
	EValidatorResult Result = Validator.IsValid(PropertyName);
	if (Result != EValidatorResult::Ok)
	{
		const FString ErrorMessage = FKismetNameValidator::GetErrorString(PropertyName, Result);
		UKismetSystemLibrary::RaiseScriptError(FString::Printf(TEXT("Failed to add ViewModel property '%s' on ViewModel '%s'. %s"), *PropertyName, *ViewModel->GetPathName(), *ErrorMessage));
		return false;
	}

	FName PinTypeCategory = *PropertyType;
	FName PinTypeSubCategory = NAME_None;

	// Add support for float/double. PC_Real is the pin category, Float and Double are the SubCategory
	if (PropertyType == UEdGraphSchema_K2::PC_Float || PropertyType == UEdGraphSchema_K2::PC_Double)
	{
		PinTypeCategory = UEdGraphSchema_K2::PC_Real;
		PinTypeSubCategory = *PropertyType;
	}

	FEdGraphPinType PinType = UBlueprintEditorLibrary::GetBasicTypeByName(PinTypeCategory);
	if (!PinTypeSubCategory.IsNone())
	{
		PinType.PinSubCategory = PinTypeSubCategory;
	}
	
	// GetBasicTypeByName defaults to int. If int was not provided, fail
	if (PinType.PinCategory == UEdGraphSchema_K2::PC_Int && PropertyType != UEdGraphSchema_K2::PC_Int)
	{
		UKismetSystemLibrary::RaiseScriptError(FString::Printf(TEXT("Failed to add ViewModel property '%s' on ViewModel '%s'. Unsupported property type '%s'"), *PropertyName, *ViewModel->GetPathName(), *PropertyType));
		return false;
	}

	if (!FBlueprintEditorUtils::AddMemberVariable(ViewModel, *PropertyName, PinType, DefaultValue))
	{
		UKismetSystemLibrary::RaiseScriptError(FString::Printf(TEXT("Failed to add ViewModel property '%s' on ViewModel '%s'."), *PropertyName, *ViewModel->GetPathName()));
		return false;
	}

	// Mark variable with FieldNotify metadata
	FBlueprintEditorUtils::SetBlueprintVariableMetaData(ViewModel, *PropertyName, nullptr, FBlueprintMetadata::MD_FieldNotify, TEXT(""));
	FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(ViewModel);
	return true;
}

TArray<UClass*> UMVVMToolset::ListViewModels(const FString& SearchPath)
{
	// Get native ViewModels
	TArray<UClass*> Results;
	GetDerivedClasses(UMVVMViewModelBase::StaticClass(), Results, /*bRecursive=*/true);

	// Get Blueprint ViewModels

	FARFilter Filter;
	Filter.ClassPaths.Add(UMVVMViewModelBase::StaticClass()->GetClassPathName());
	Filter.bRecursiveClasses = true;

	if (!SearchPath.IsEmpty())
	{
		Filter.PackagePaths.Add(FName(*SearchPath));
	}

	TArray<FAssetData> AssetList;
	UAssetRegistryHelpers::GetBlueprintAssets(Filter, AssetList);
	for (const FAssetData& AssetData : AssetList)
	{		
		Results.Add(AssetData.GetClass());
	}

	return Results;
}

TArray<UClass*> UMVVMToolset::ListWidgetViewModels(UWidgetBlueprint* WidgetBlueprint)
{	
	UMVVMBlueprintView* View = GetWidgetView(WidgetBlueprint);
	if (!IsValid(View))
	{
		UKismetSystemLibrary::RaiseScriptError(FString::Printf(TEXT("Failed to set list ViewModels used by widget '%s'. Invalid MVVM View."), *WidgetBlueprint->GetPathName()));
		return TArray<UClass*>();
	}

	TArray<UClass*> Results;
	for(const FMVVMBlueprintViewModelContext& ViewModelContext : View->GetViewModels())
	{
		if (UClass* Class = ViewModelContext.GetViewModelClass())
		{
			Results.Add(Class);
		}
	}
	return Results;
}

void UMVVMToolset::AddViewModelToWidget(UWidgetBlueprint* WidgetBlueprint, UClass* ViewModelClass)
{
	UMVVMBlueprintView* View = GetWidgetView(WidgetBlueprint);
	if (!IsValid(View))
	{
		UKismetSystemLibrary::RaiseScriptError(FString::Printf(TEXT("Failed to add ViewModel to widget '%s'. Invalid MVVM View."), *WidgetBlueprint->GetPathName()));
		return;
	}

	UMVVMEditorSubsystem* MVVMEditorSubsystem = GEditor->GetEditorSubsystem<UMVVMEditorSubsystem>();
	if (!IsValid(MVVMEditorSubsystem))
	{
		UKismetSystemLibrary::RaiseScriptError(FString::Printf(TEXT("Failed to remove binding. UMVVMEditorSubsystem not loaded.")));
		return;
	}

	MVVMEditorSubsystem->AddViewModel(WidgetBlueprint, ViewModelClass);
}

TArray<FMVVMBlueprintViewBinding> UMVVMToolset::ListWidgetViewBindings(UWidgetBlueprint* WidgetBlueprint)
{	
	UMVVMBlueprintView* View = GetWidgetView(WidgetBlueprint);
	if (!IsValid(View))
	{
		UKismetSystemLibrary::RaiseScriptError(FString::Printf(TEXT("Failed to get View for widget '%s'."), *WidgetBlueprint->GetPathName()));
		return TArray<FMVVMBlueprintViewBinding>();
	}

	return TArray<FMVVMBlueprintViewBinding>(View->GetBindings());
}

bool UMVVMToolset::RemoveWidgetViewBinding(UWidgetBlueprint* WidgetBlueprint, const FGuid& BindingId)
{
	UMVVMEditorSubsystem* MVVMEditorSubsystem = GEditor->GetEditorSubsystem<UMVVMEditorSubsystem>();
	if (!IsValid(MVVMEditorSubsystem))
	{
		UKismetSystemLibrary::RaiseScriptError(FString::Printf(TEXT("Failed to remove binding. UMVVMEditorSubsystem not loaded.")));
		return false;
	}

	FString ErrorMessage;
	UMVVMBlueprintView* View = GetWidgetView(WidgetBlueprint);
	if (!IsValid(View))
	{
		UKismetSystemLibrary::RaiseScriptError(FString::Printf(TEXT("Failed to remove binding. Invalid MVVM View")));
		return false;
	}

	FMVVMBlueprintViewBinding* Binding = View->GetBinding(BindingId);
	if (Binding == nullptr)
	{	
		UKismetSystemLibrary::RaiseScriptError(FString::Printf(TEXT("Failed to remove binding. Invalid BindingId '%s'."), *BindingId.ToString()));
		return false;
	}

	MVVMEditorSubsystem->RemoveBinding(WidgetBlueprint, *Binding);
	return View->GetBinding(BindingId) == nullptr;
}

FGuid UMVVMToolset::CreateViewBinding(UWidgetBlueprint* WidgetBlueprint, UObject* SourceContext, const FString& SourcePropertyPath, UObject* DestinationContext, const FString& DestinationPropertyPath, FName ConversionName)
{
	FString Error;
	const FMVVMBlueprintPropertyPath SourcePath = GetPropertyPath(WidgetBlueprint, SourceContext, SourcePropertyPath, Error);
	if(!SourcePath.IsValid())
	{
		UKismetSystemLibrary::RaiseScriptError(FString::Printf(TEXT("Failed to create view binding. %s"), *Error));
		return FGuid();
	}

	const FMVVMBlueprintPropertyPath DestinationPath = GetPropertyPath(WidgetBlueprint, DestinationContext, DestinationPropertyPath, Error);
	if(!DestinationPath.IsValid())
	{
		UKismetSystemLibrary::RaiseScriptError(FString::Printf(TEXT("Failed to create view binding. %s"), *Error));
		return FGuid();
	}

	UMVVMEditorSubsystem* MVVMEditorSubsystem = GEditor->GetEditorSubsystem<UMVVMEditorSubsystem>();
	if (!IsValid(MVVMEditorSubsystem))
	{
		UKismetSystemLibrary::RaiseScriptError(FString::Printf(TEXT("Failed to create binding. UMVVMEditorSubsystem not loaded.")));
		return FGuid();
	}

	UMVVMBlueprintView* View = GetWidgetView(WidgetBlueprint);
	if (!IsValid(View))
	{
		UKismetSystemLibrary::RaiseScriptError(FString::Printf(TEXT("Failed to create binding. Missing View.")));
		return FGuid();
	}

	FMVVMBlueprintViewBinding& NewBinding = MVVMEditorSubsystem->AddBinding(WidgetBlueprint);
	MVVMEditorSubsystem->SetSourcePathForBinding(WidgetBlueprint, NewBinding, SourcePath);
	MVVMEditorSubsystem->SetDestinationPathForBinding(WidgetBlueprint, NewBinding, DestinationPath, false);

	TryAutoConvertBindingFromSourceToDestination(WidgetBlueprint, NewBinding, ConversionName);
	return NewBinding.BindingId;
}

TArray<FMVVMViewConversionFunctionDescription> UMVVMToolset::ListConversionFunctions(UWidgetBlueprint* WidgetBlueprint)
{
	UMVVMEditorSubsystem* MVVMEditorSubsystem = GEditor->GetEditorSubsystem<UMVVMEditorSubsystem>();
	if (!IsValid(MVVMEditorSubsystem))
	{
		UKismetSystemLibrary::RaiseScriptError(FString::Printf(TEXT("Failed to get available conversion functions for widget %s. UMVVMEditorSubsystem not loaded."), *WidgetBlueprint->GetName()));
		return TArray<FMVVMViewConversionFunctionDescription>();
	}
	
	TArray<FMVVMViewConversionFunctionDescription> Results;
	for (const UE::MVVM::FConversionFunctionValue& ConversionFunction : MVVMEditorSubsystem->GetConversionFunctions(WidgetBlueprint, nullptr, nullptr))
	{
		FMVVMViewConversionFunctionDescription& Description = Results.Emplace_GetRef();
		Description.ConversionFunction = ConversionFunction.GetFunction();
		Description.ConversionNode = ConversionFunction.GetNode();
	}
	return Results;
}