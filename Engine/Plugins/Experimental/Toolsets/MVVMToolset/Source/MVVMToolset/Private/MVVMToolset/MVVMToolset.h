// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MVVMBlueprintView.h"
#include "ToolsetRegistry/UToolsetRegistry.h"
#include "Types/MVVMConversionFunctionValue.h"
#include "UObject/UnrealType.h"

#include "MVVMToolset.generated.h"

/* Information about a conversion function*/
USTRUCT(BlueprintType)
struct FMVVMViewConversionFunctionDescription
{
	GENERATED_BODY()

	/* If the conversion function is a UFunction, this value will be set */
	UPROPERTY(BlueprintReadOnly, Category = "MVVM")
	TObjectPtr<const UFunction> ConversionFunction;

	/* If the conversion function is a UK2Node, this value will be set */
	UPROPERTY(BlueprintReadOnly, Category = "MVVM")
	TSubclassOf<UK2Node> ConversionNode;
};

UCLASS(BlueprintType, Hidden)
class UMVVMToolset : public UToolsetDefinition
{
	GENERATED_BODY()

public:	
	/* 
	Creates a new ViewModel asset
		
	@param ViewModelName - Desired ViewModel asset name. 
	@param PackagePath - Target path in which the asset should reside
	@param ParentClass - Class of which the ViewModel should derive. Must derive UMVVMViewModelBase 
	@return newly created ViewModel blueprint.
	*/ 
	UFUNCTION(BlueprintCallable, meta = (Experimental, AICallable), Category = "MVVM")
	static UBlueprint* CreateViewModel(const FString& ViewModelName, const FString& PackagePath, UClass* ParentClass);

	/* 
	Add a property to an existing ViewModel
		
	@param ViewModel - ViewModel asset to use. 
	@param PropertyName - Desired name of the new property 
	@param PropertyType - Desired type of the new property. Corresponds with PinType
	@param DefaultValue - If a default property is desired, string representation of the value
	@return True if property was succesfully added.
	*/ 
	UFUNCTION(BlueprintCallable, meta = (Experimental, AICallable), Category = "MVVM")
	static bool AddViewModelProperty(UBlueprint* ViewModel, const FString& PropertyName, const FString& PropertyType, const FString& DefaultValue);

	/* 
	List all ViewModel under the search path
		
	@param SearchPath - Mount point search path for ViewModel assets
	@return List of ViewModel classes.
	*/ 
	UFUNCTION(BlueprintCallable, meta = (Experimental, AICallable), Category = "MVVM")
	static TArray<UClass*> ListViewModels(const FString& SearchPath);

	/* 
	List all ViewModel on the WidgetBlueprint
		
	@param WidgetBlueprint - WidgetBlueprint to use
	@return List of ViewModel classes.
	*/ 
	UFUNCTION(BlueprintCallable, meta = (Experimental, AICallable), Category = "MVVM")
	static TArray<UClass*> ListWidgetViewModels(UWidgetBlueprint* WidgetBlueprint);

	/* 
	Adds a ViewModel to the WidgetBlueprint
		
	@param WidgetBlueprint - WidgetBlueprint to use
	@param ViewModelClass - ViewModel class to use.
	*/ 
	UFUNCTION(BlueprintCallable, meta = (Experimental, AICallable), Category = "MVVM")
	static void AddViewModelToWidget(UWidgetBlueprint* WidgetBlueprint, UClass* ViewModelClass);

	/* 
	List all view bindings on the WidgetBlueprint
		
	@param WidgetBlueprint - WidgetBlueprint to use
	@return List of FMVVMBlueprintViewBinding that describe each available view binding.
	*/ 
	UFUNCTION(BlueprintCallable, meta = (Experimental, AICallable), Category = "MVVM")
	static TArray<FMVVMBlueprintViewBinding> ListWidgetViewBindings(UWidgetBlueprint* WidgetBlueprint);

	/* 
	Removes View Binding on the WidgetBlueprint
		
	@param WidgetBlueprint - WidgetBlueprint to use
	@return True if binding was removed.
	*/ 
	UFUNCTION(BlueprintCallable, meta = (Experimental, AICallable), Category = "MVVM")
	static bool RemoveWidgetViewBinding(UWidgetBlueprint* WidgetBlueprint, const FGuid& BindingID);

	/* 
	Creates a View Binding from the SourceProperty to the Destination property. 
	If there is a type mismatch, an existing conversion function will try to be used.
	
	@param WidgetBlueprint - WidgetBlueprint to use
	@param SourceContext - Source object that contains the source property. 
		Can be the following:
		- UWidget: source property is from a widget in the widget blueprint's widgettree
		- UClass: source property is from a widget blueprint's ViewModel 
		- Null: source property is from the widget blueprint
	@param SourcePropertyPath - Property to read data from. Contains a dot '.' separated path to the field (path.subfield)
	@param DestinationContext - Destination object that contains the destination property. 
		Can be the following:
		- UWidget: destination property is from a widget in the widget blueprint's widgettree
		- UClass: destination property is from a widget blueprint's ViewModel 
		- Null: destination property is from the widget blueprint
	@param DestinationPropertyPath - Property to write data to. Contains a dot '.' separated path to the field (path.subfield)
	@param ConversionName - Optional parameter, name of a conversion function to use if needed. If none is provided, this will be inferred.
	@return Valid FGuid if the binding was created
	*/
	UFUNCTION(BlueprintCallable, meta = (Experimental, AICallable), Category = "MVVM")
	static FGuid CreateViewBinding(UWidgetBlueprint* WidgetBlueprint, UObject* SourceContext, const FString& SourcePropertyPath, UObject* DestinationContext, const FString& DestinationPropertyPath, FName ConversionName);

	/* 
	List Available Conversion functions for a WidgetBlueprint that bind a Source Property to a Destination Property
	@param WidgetBlueprint - WidgetBlueprint to use
	@return the FMVVMViewConversionFunctionDescription for available conversion functions. 
	*/
	UFUNCTION(BlueprintCallable, meta = (Experimental, AICallable), Category = "MVVM")
	static TArray<FMVVMViewConversionFunctionDescription> ListConversionFunctions(UWidgetBlueprint* WidgetBlueprint);
};
