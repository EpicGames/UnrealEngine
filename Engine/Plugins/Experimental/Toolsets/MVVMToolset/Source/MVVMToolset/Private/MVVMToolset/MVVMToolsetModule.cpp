// Copyright Epic Games, Inc. All Rights Reserved.

#include "Editor.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "ToolsetRegistry/ToolsetRegistrySubsystem.h"
#include "ToolsetRegistry/UToolsetRegistry.h"

#include "MVVMToolset.h"

class FMVVMToolsetModule : public IModuleInterface
{
public:
	virtual void StartupModule() override
	{
		UToolsetRegistry::RegisterToolsetClass(UMVVMToolset::StaticClass());
	}

	virtual void ShutdownModule() override
	{
		UToolsetRegistry::UnregisterToolsetClass(UMVVMToolset::StaticClass());
	}
};

IMPLEMENT_MODULE(FMVVMToolsetModule, MVVMToolset);
