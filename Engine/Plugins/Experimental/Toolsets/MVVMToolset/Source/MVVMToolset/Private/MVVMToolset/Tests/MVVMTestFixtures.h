// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MVVMViewModelBase.h"
#include "MVVMTestFixtures.generated.h"

/**
 * Minimal ViewModel for use as a custom parent class in CreateViewModel tests.
 * Lets tests verify that CreateViewModel respects a non-default parent class
 * without depending on any game-specific Blueprint assets.
 */
UCLASS(Transient, Hidden)
class UMVVMTestBaseViewModel : public UMVVMViewModelBase
{
	GENERATED_BODY()
};
