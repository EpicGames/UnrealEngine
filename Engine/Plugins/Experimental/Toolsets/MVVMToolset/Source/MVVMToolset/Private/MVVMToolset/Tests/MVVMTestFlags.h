// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_DEV_AUTOMATION_TESTS

#include "AssetRegistry/AssetRegistryModule.h"
#include "Blueprint/UserWidget.h"
#include "Editor.h"
#include "Engine/Engine.h"
#include "Misc/AutomationTest.h"
#include "Misc/PackageName.h"
#include "Misc/Paths.h"
#include "MVVMEditorSubsystem.h"
#include "WidgetBlueprint.h"
#include "WidgetBlueprintFactory.h"

namespace MVVMToolSetTest
{
	static const auto Flags =
		EAutomationTestFlags::EditorContext |
		EAutomationTestFlags::ProductFilter |
		EAutomationTestFlags::CriticalPriority;

	static const FString TestMountPoint = TEXT("/Automation/MVVMToolSetTest/");

	inline void RegisterTestMountPoint()
	{
		FPackageName::RegisterMountPoint(*TestMountPoint, FPaths::AutomationTransientDir());
	}

	inline void UnregisterTestMountPoint()
	{
		FPackageName::UnRegisterMountPoint(*TestMountPoint, FPaths::AutomationTransientDir());
		if (GEngine)
		{
			GEngine->ForceGarbageCollection(true);
		}
	}

	inline UWidgetBlueprint* CreateTestWidgetBlueprintWithMVVM(const FString& Name)
	{
		const FString FullPackageName = TestMountPoint + Name;
		UPackage* Package = CreatePackage(*FullPackageName);
		if (!IsValid(Package))
		{
			return nullptr;
		}

		UWidgetBlueprintFactory* Factory = NewObject<UWidgetBlueprintFactory>();
		Factory->ParentClass = UUserWidget::StaticClass();
		UWidgetBlueprint* WidgetBlueprint = Cast<UWidgetBlueprint>(Factory->FactoryCreateNew(UWidgetBlueprint::StaticClass(), Package, FName(*Name), RF_Public | RF_Standalone, nullptr, GWarn));
		if (!IsValid(WidgetBlueprint))
		{
			return nullptr;
		}

		FAssetRegistryModule::AssetCreated(WidgetBlueprint);
		return WidgetBlueprint;
	}
}

#endif // WITH_DEV_AUTOMATION_TESTS
