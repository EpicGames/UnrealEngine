// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_DEV_AUTOMATION_TESTS

#include "MVVMBlueprintViewBinding.h"
#include "MVVMTestFlags.h"
#include "MVVMTestFixtures.h"
#include "MVVMToolset/MVVMToolset.h"
#include "WidgetBlueprint.h"
#include "ToolsetRegistry/UToolsetRegistry.h"

BEGIN_DEFINE_SPEC(FMVVMTest_WorkingPath,
	"AI.Toolsets.MVVM.WorkingPath", MVVMToolSetTest::Flags)
	UWidgetBlueprint* WidgetBlueprint = nullptr;
	UBlueprint* ViewModel = nullptr;
	int32 TestCounter = 0;
END_DEFINE_SPEC(FMVVMTest_WorkingPath)

void FMVVMTest_WorkingPath::Define()
{
	BeforeEach([this]()
	{
		MVVMToolSetTest::RegisterTestMountPoint();
		FString ViewModelName = FString::Printf(TEXT("VM_WorkingPath_%d"), TestCounter);
		ViewModel = UMVVMToolset::CreateViewModel(ViewModelName, MVVMToolSetTest::TestMountPoint, nullptr);
		FString WidgetBlueprintName = FString::Printf(TEXT("WWidgetBlueprint_WorkingPath_%d"), TestCounter++);
		WidgetBlueprint = MVVMToolSetTest::CreateTestWidgetBlueprintWithMVVM(WidgetBlueprintName);
		
		if (IsValid(WidgetBlueprint) && IsValid(ViewModel))
		{
			UMVVMToolset::AddViewModelToWidget(WidgetBlueprint, ViewModel->SkeletonGeneratedClass);
		}

		if (IsValid(ViewModel))
		{
			//Assumed ViewModel variables 
			UMVVMToolset::AddViewModelProperty(ViewModel, TEXT("Health"), TEXT("float"), TEXT(""));
		}
	});

	AfterEach([this]()
	{
		WidgetBlueprint = nullptr;
		ViewModel = nullptr;
		MVVMToolSetTest::UnregisterTestMountPoint();
	});

	// ---- CreateViewModel ----

	It("CreateViewModel returns non-null blueprint", [this]()
	{
		TestNotNull(TEXT("ViewModel created"), ViewModel);
	});

	It("CreateViewModel with native parent class succeeds", [this]()
	{
		FString Name = FString::Printf(TEXT("VM_CustomParent_%d"), TestCounter);
		UBlueprint* CustomViewModel = UMVVMToolset::CreateViewModel(Name, MVVMToolSetTest::TestMountPoint, UMVVMTestBaseViewModel::StaticClass());
		TestNotNull(TEXT("ViewModel created with custom parent"), CustomViewModel);
	});

	// ---- AddViewModelProperty ----

	It("AddViewModelProperty returns true for a valid float property", [this]()
	{
		TestNotNull(TEXT("ViewModel valid"), ViewModel);
		if (!IsValid(ViewModel))
		{
			AddInfo(TEXT("Skipped: [Invalid Test ViewModel Blueprint]"));
			return;
		}

		bool bAdded = UMVVMToolset::AddViewModelProperty(ViewModel, TEXT("Stamina"), TEXT("float"), TEXT(""));
		TestTrue(TEXT("Property added"), bAdded);
	});

	It("AddViewModelProperty returns true for a valid bool property with default value", [this]()
	{
		TestNotNull(TEXT("ViewModel valid"), ViewModel);
		if (!IsValid(ViewModel))
		{
			AddInfo(TEXT("Skipped: [Invalid Test ViewModel Blueprint]"));
			return;
		}

		bool bAdded = UMVVMToolset::AddViewModelProperty(ViewModel, TEXT("bIsAlive"), TEXT("bool"), TEXT("true"));
		TestTrue(TEXT("Bool property added"), bAdded);
	});

	It("AddViewModelProperty adds multiple distinct properties", [this]()
	{
		TestNotNull(TEXT("ViewModel valid"), ViewModel);
		if (!IsValid(ViewModel))
		{
			AddInfo(TEXT("Skipped: [Invalid Test ViewModel Blueprint]"));
			return;
		}

		bool bFirst  = UMVVMToolset::AddViewModelProperty(ViewModel, TEXT("Stamina"), TEXT("float"), TEXT(""));
		bool bSecond = UMVVMToolset::AddViewModelProperty(ViewModel, TEXT("Shield"),  TEXT("float"), TEXT(""));
		TestTrue(TEXT("First property added"),  bFirst);
		TestTrue(TEXT("Second property added"), bSecond);
	});

	// ---- ListViewModels ----

	It("ListViewModels includes the created ViewModel", [this]()
	{
		TArray<UClass*> ViewModels = UMVVMToolset::ListViewModels(MVVMToolSetTest::TestMountPoint);
		TestTrue(TEXT("At least one ViewModel found"), ViewModels.Num() > 0);
	});

	It("ListViewModels with empty search path returns all ViewModels", [this]()
	{
		TArray<UClass*> ViewModels = UMVVMToolset::ListViewModels(TEXT(""));
		TestTrue(TEXT("Non-empty result with empty search path"), ViewModels.Num() > 0);
	});

	It("ListViewModels with returns native ViewModels", [this]()
	{
		TArray<UClass*> ViewModels = UMVVMToolset::ListViewModels(TEXT(""));
		TestTrue(TEXT("Native ViewModel class in results"), ViewModels.Contains(UMVVMTestBaseViewModel::StaticClass()));
	});

	// ---- WidgetBlueprint with MVVM ----

	It("CreateTestWidgetBlueprintWithMVVM produces a valid blueprint", [this]()
	{
		TestNotNull(TEXT("Widget WidgetBlueprint created"), WidgetBlueprint);
	});

	// ---- AddViewModelToWidget ----

	It("AddViewModelToWidget makes the ViewModel visible via ListWidgetViewModels", [this]()
	{
		TestNotNull(TEXT("WidgetBlueprint valid"), WidgetBlueprint);
		if (!IsValid(WidgetBlueprint))
		{
			AddInfo(TEXT("Skipped: [Invalid Test Widget Blueprint]"));
			return;
		}

		// BeforeEach already called AddViewModelToWidget — verify it registered
		TArray<UClass*> ViewModels = UMVVMToolset::ListWidgetViewModels(WidgetBlueprint);
		TestTrue(TEXT("ViewModel visible on widget"), ViewModels.Num() > 0);
	});

	It("AddViewModelToWidget makes native ViewModel visible via ListWidgetViewModels", [this]()
	{
		FString FreshName = FString::Printf(TEXT("WWidgetBlueprint_NoVM_%d"), TestCounter);
		UWidgetBlueprint* FreshWidget = MVVMToolSetTest::CreateTestWidgetBlueprintWithMVVM(FreshName);
		TestNotNull(TEXT("Fresh widget created"), FreshWidget);
		if (!IsValid(FreshWidget))
		{
			AddInfo(TEXT("Skipped: [Could not create fresh widget]"));
			return;
		}

		UMVVMToolset::AddViewModelToWidget(FreshWidget, UMVVMTestBaseViewModel::StaticClass());

		TArray<UClass*> ViewModels = UMVVMToolset::ListWidgetViewModels(FreshWidget);
		TestTrue(TEXT("Native ViewModel on fresh widget"), ViewModels.Contains(UMVVMTestBaseViewModel::StaticClass()));
	});


	// ---- ListWidgetViewModels ----

	It("ListWidgetViewModels returns empty for widget with no ViewModels assigned", [this]()
	{
		FString FreshName = FString::Printf(TEXT("WWidgetBlueprint_NoVM_%d"), TestCounter);
		UWidgetBlueprint* FreshWidget = MVVMToolSetTest::CreateTestWidgetBlueprintWithMVVM(FreshName);
		TestNotNull(TEXT("Fresh widget created"), FreshWidget);
		if (!IsValid(FreshWidget))
		{
			AddInfo(TEXT("Skipped: [Could not create fresh widget]"));
			return;
		}

		TArray<UClass*> ViewModels = UMVVMToolset::ListWidgetViewModels(FreshWidget);
		TestEqual(TEXT("No ViewModels on fresh widget"), ViewModels.Num(), 0);
	});

	// ---- ListWidgetViewBindings ----

	It("ListWidgetViewBindings returns empty array on fresh blueprint", [this]()
	{
		TestNotNull(TEXT("WidgetBlueprint valid"), WidgetBlueprint);
		if (!IsValid(WidgetBlueprint))
		{
			AddInfo(TEXT("Skipped: [Invalid Test Widget Blueprint]"));
			return;
		}

		TArray<FMVVMBlueprintViewBinding> Bindings = UMVVMToolset::ListWidgetViewBindings(WidgetBlueprint);
		TestEqual(TEXT("No bindings on fresh blueprint"), Bindings.Num(), 0);
	});

	// ---- CreateViewBinding ----

	It("CreateViewBinding returns a valid GUID", [this]()
	{
		TestNotNull(TEXT("WidgetBlueprint valid"), WidgetBlueprint);
		TestNotNull(TEXT("ViewModel valid"), ViewModel);
		if (!IsValid(WidgetBlueprint) || !IsValid(ViewModel) || !IsValid(ViewModel->SkeletonGeneratedClass))
		{
			AddInfo(TEXT("Skipped: [Invalid test fixtures]"));
			return;
		}

		FString VM2Name = FString::Printf(TEXT("VM_BindDest_%d"), TestCounter);
		UBlueprint* ViewModel2 = UMVVMToolset::CreateViewModel(VM2Name, MVVMToolSetTest::TestMountPoint, nullptr);
		if (!IsValid(ViewModel2) || !IsValid(ViewModel2->SkeletonGeneratedClass))
		{
			AddInfo(TEXT("Skipped: [Could not create second ViewModel]"));
			return;
		}

		UMVVMToolset::AddViewModelProperty(ViewModel2, TEXT("Health"), TEXT("float"), TEXT(""));
		UMVVMToolset::AddViewModelToWidget(WidgetBlueprint, ViewModel2->SkeletonGeneratedClass);

		FGuid BindingId = UMVVMToolset::CreateViewBinding(
			WidgetBlueprint,
			ViewModel->SkeletonGeneratedClass,  TEXT("Health"),
			ViewModel2->SkeletonGeneratedClass, TEXT("Health"),
			NAME_None);
		TestTrue(TEXT("Binding GUID is valid"), BindingId.IsValid());
	});

	It("CreateViewBinding adds a binding visible via ListWidgetViewBindings", [this]()
	{
		TestNotNull(TEXT("WidgetBlueprint valid"), WidgetBlueprint);
		TestNotNull(TEXT("ViewModel valid"), ViewModel);
		if (!IsValid(WidgetBlueprint) || !IsValid(ViewModel) || !IsValid(ViewModel->SkeletonGeneratedClass))
		{
			AddInfo(TEXT("Skipped: [Invalid test fixtures]"));
			return;
		}

		FString VM2Name = FString::Printf(TEXT("VM_BindDestList_%d"), TestCounter);
		UBlueprint* ViewModel2 = UMVVMToolset::CreateViewModel(VM2Name, MVVMToolSetTest::TestMountPoint, nullptr);
		if (!IsValid(ViewModel2) || !IsValid(ViewModel2->SkeletonGeneratedClass))
		{
			AddInfo(TEXT("Skipped: [Could not create second ViewModel]"));
			return;
		}
		UMVVMToolset::AddViewModelProperty(ViewModel2, TEXT("Health"), TEXT("float"), TEXT(""));
		UMVVMToolset::AddViewModelToWidget(WidgetBlueprint, ViewModel2->SkeletonGeneratedClass);

		UMVVMToolset::CreateViewBinding(
			WidgetBlueprint,
			ViewModel->SkeletonGeneratedClass,  TEXT("Health"),
			ViewModel2->SkeletonGeneratedClass, TEXT("Health"),
			NAME_None);

		TArray<FMVVMBlueprintViewBinding> Bindings = UMVVMToolset::ListWidgetViewBindings(WidgetBlueprint);
		TestEqual(TEXT("One binding exists after create"), Bindings.Num(), 1);
	});

	It("CreateViewBinding ID matches what ListWidgetViewBindings reports", [this]()
	{
		TestNotNull(TEXT("WidgetBlueprint valid"), WidgetBlueprint);
		TestNotNull(TEXT("ViewModel valid"), ViewModel);
		if (!IsValid(WidgetBlueprint) || !IsValid(ViewModel) || !IsValid(ViewModel->SkeletonGeneratedClass))
		{
			AddInfo(TEXT("Skipped: [Invalid test fixtures]"));
			return;
		}

		FString VM2Name = FString::Printf(TEXT("VM_BindIdCheck_%d"), TestCounter);
		UBlueprint* ViewModel2 = UMVVMToolset::CreateViewModel(VM2Name, MVVMToolSetTest::TestMountPoint, nullptr);
		if (!IsValid(ViewModel2) || !IsValid(ViewModel2->SkeletonGeneratedClass))
		{
			AddInfo(TEXT("Skipped: [Could not create second ViewModel]"));
			return;
		}
		UMVVMToolset::AddViewModelProperty(ViewModel2, TEXT("Health"), TEXT("float"), TEXT(""));
		UMVVMToolset::AddViewModelToWidget(WidgetBlueprint, ViewModel2->SkeletonGeneratedClass);

		FGuid CreatedId = UMVVMToolset::CreateViewBinding(
			WidgetBlueprint,
			ViewModel->SkeletonGeneratedClass,  TEXT("Health"),
			ViewModel2->SkeletonGeneratedClass, TEXT("Health"),
			NAME_None);

		TArray<FMVVMBlueprintViewBinding> Bindings = UMVVMToolset::ListWidgetViewBindings(WidgetBlueprint);
		TestEqual(TEXT("One binding"), Bindings.Num(), 1);
		if (Bindings.Num() > 0)
		{
			TestEqual(TEXT("Binding IDs match"), Bindings[0].BindingId, CreatedId);
		}
	});

	It("Multiple CreateViewBinding calls each produce a unique binding", [this]()
	{
		TestNotNull(TEXT("WidgetBlueprint valid"), WidgetBlueprint);
		TestNotNull(TEXT("ViewModel valid"), ViewModel);
		if (!IsValid(WidgetBlueprint) || !IsValid(ViewModel) || !IsValid(ViewModel->SkeletonGeneratedClass))
		{
			AddInfo(TEXT("Skipped: [Invalid test fixtures]"));
			return;
		}

		FString VM2Name = FString::Printf(TEXT("VM_BindUniq_%d"), TestCounter);
		UBlueprint* ViewModel2 = UMVVMToolset::CreateViewModel(VM2Name, MVVMToolSetTest::TestMountPoint, nullptr);
		if (!IsValid(ViewModel2) || !IsValid(ViewModel2->SkeletonGeneratedClass))
		{
			AddInfo(TEXT("Skipped: [Could not create second ViewModel]"));
			return;
		}
		UMVVMToolset::AddViewModelProperty(ViewModel2, TEXT("Health"), TEXT("float"), TEXT(""));
		UMVVMToolset::AddViewModelToWidget(WidgetBlueprint, ViewModel2->SkeletonGeneratedClass);

		FGuid IdA = UMVVMToolset::CreateViewBinding(
			WidgetBlueprint,
			ViewModel->SkeletonGeneratedClass,  TEXT("Health"),
			ViewModel2->SkeletonGeneratedClass, TEXT("Health"),
			NAME_None);
		FGuid IdB = UMVVMToolset::CreateViewBinding(
			WidgetBlueprint,
			ViewModel->SkeletonGeneratedClass,  TEXT("Health"),
			ViewModel2->SkeletonGeneratedClass, TEXT("Health"),
			NAME_None);

		TestNotEqual(TEXT("Binding IDs are unique"), IdA, IdB);

		TArray<FMVVMBlueprintViewBinding> Bindings = UMVVMToolset::ListWidgetViewBindings(WidgetBlueprint);
		TestEqual(TEXT("Two bindings exist"), Bindings.Num(), 2);
	});

	// ---- RemoveWidgetViewBinding ----

	It("RemoveWidgetViewBinding returns true and removes the binding", [this]()
	{
		TestNotNull(TEXT("WidgetBlueprint valid"), WidgetBlueprint);
		TestNotNull(TEXT("ViewModel valid"), ViewModel);
		if (!IsValid(WidgetBlueprint) || !IsValid(ViewModel) || !IsValid(ViewModel->SkeletonGeneratedClass))
		{
			AddInfo(TEXT("Skipped: [Invalid test fixtures]"));
			return;
		}

		FString VM2Name = FString::Printf(TEXT("VM_RemoveTest_%d"), TestCounter);
		UBlueprint* ViewModel2 = UMVVMToolset::CreateViewModel(VM2Name, MVVMToolSetTest::TestMountPoint, nullptr);
		if (!IsValid(ViewModel2) || !IsValid(ViewModel2->SkeletonGeneratedClass))
		{
			AddInfo(TEXT("Skipped: [Could not create second ViewModel]"));
			return;
		}
		UMVVMToolset::AddViewModelProperty(ViewModel2, TEXT("Health"), TEXT("float"), TEXT(""));
		UMVVMToolset::AddViewModelToWidget(WidgetBlueprint, ViewModel2->SkeletonGeneratedClass);

		FGuid BindingId = UMVVMToolset::CreateViewBinding(
			WidgetBlueprint,
			ViewModel->SkeletonGeneratedClass,  TEXT("Health"),
			ViewModel2->SkeletonGeneratedClass, TEXT("Health"),
			NAME_None);

		bool bRemoved = UMVVMToolset::RemoveWidgetViewBinding(WidgetBlueprint, BindingId);
		TestTrue(TEXT("Remove returned true"), bRemoved);

		TArray<FMVVMBlueprintViewBinding> Bindings = UMVVMToolset::ListWidgetViewBindings(WidgetBlueprint);
		TestEqual(TEXT("Binding list is empty after remove"), Bindings.Num(), 0);
	});

	It("RemoveWidgetViewBinding removes only the targeted binding", [this]()
	{
		TestNotNull(TEXT("WidgetBlueprint valid"), WidgetBlueprint);
		TestNotNull(TEXT("ViewModel valid"), ViewModel);
		if (!IsValid(WidgetBlueprint) || !IsValid(ViewModel) || !IsValid(ViewModel->SkeletonGeneratedClass))
		{
			AddInfo(TEXT("Skipped: [Invalid test fixtures]"));
			return;
		}

		FString VM2Name = FString::Printf(TEXT("VM_RemoveSelective_%d"), TestCounter);
		UBlueprint* ViewModel2 = UMVVMToolset::CreateViewModel(VM2Name, MVVMToolSetTest::TestMountPoint, nullptr);
		if (!IsValid(ViewModel2) || !IsValid(ViewModel2->SkeletonGeneratedClass))
		{
			AddInfo(TEXT("Skipped: [Could not create second ViewModel]"));
			return;
		}
		UMVVMToolset::AddViewModelProperty(ViewModel2, TEXT("Health"), TEXT("float"), TEXT(""));
		UMVVMToolset::AddViewModelToWidget(WidgetBlueprint, ViewModel2->SkeletonGeneratedClass);

		FGuid IdA = UMVVMToolset::CreateViewBinding(
			WidgetBlueprint,
			ViewModel->SkeletonGeneratedClass,  TEXT("Health"),
			ViewModel2->SkeletonGeneratedClass, TEXT("Health"),
			NAME_None);
		FGuid IdB = UMVVMToolset::CreateViewBinding(
			WidgetBlueprint,
			ViewModel->SkeletonGeneratedClass,  TEXT("Health"),
			ViewModel2->SkeletonGeneratedClass, TEXT("Health"),
			NAME_None);

		UMVVMToolset::RemoveWidgetViewBinding(WidgetBlueprint, IdA);

		TArray<FMVVMBlueprintViewBinding> Bindings = UMVVMToolset::ListWidgetViewBindings(WidgetBlueprint);
		TestEqual(TEXT("One binding remains"), Bindings.Num(), 1);
		if (Bindings.Num() > 0)
		{
			TestEqual(TEXT("Remaining binding is IdB"), Bindings[0].BindingId, IdB);
		}
	});
}

// ============================================================================
// Error / Negative Path Tests
// ============================================================================

BEGIN_DEFINE_SPEC(FMVVMTest_Errors,
	"AI.Toolsets.MVVM.Errors", MVVMToolSetTest::Flags)
	int32 TestCounter = 0;
END_DEFINE_SPEC(FMVVMTest_Errors)

void FMVVMTest_Errors::Define()
{
	BeforeEach([this]()
	{
		MVVMToolSetTest::RegisterTestMountPoint();
	});

	AfterEach([this]()
	{
		MVVMToolSetTest::UnregisterTestMountPoint();
	});

	// ---- CreateViewModel errors ----

	It("CreateViewModel with invalid package path returns null", [this]()
	{
		UBlueprint* BadVM = UMVVMToolset::CreateViewModel(TEXT("VM_Bad"), TEXT("not_a_valid/path!"), nullptr);
		TestNull(TEXT("Null on invalid path"), BadVM);
	});

	It("CreateViewModel with non-ViewModel parent class returns null", [this]()
	{
		UBlueprint* BadVM = UMVVMToolset::CreateViewModel(TEXT("VM_BadParent"), MVVMToolSetTest::TestMountPoint, UObject::StaticClass());
		TestNull(TEXT("Null on incompatible parent"), BadVM);
	});

	// ---- AddViewModelProperty errors ----

	It("AddViewModelProperty with null ViewModel returns false", [this]()
	{
		bool bAdded = UMVVMToolset::AddViewModelProperty(nullptr, TEXT("Prop"), TEXT("float"), TEXT(""));
		TestFalse(TEXT("False on null ViewModel"), bAdded);
	});

	It("AddViewModelProperty with invalid type string returns false", [this]()
	{
		FString ViewModelName = FString::Printf(TEXT("VM_InvalidType_%d"), TestCounter++);
		UBlueprint* ViewModel = UMVVMToolset::CreateViewModel(ViewModelName, MVVMToolSetTest::TestMountPoint, nullptr);
		if (!IsValid(ViewModel))
		{
			AddInfo(TEXT("Skipped: [Invalid Test ViewModel Blueprint]"));
			return;
		}

		bool bAdded = UMVVMToolset::AddViewModelProperty(ViewModel, TEXT("Prop"), TEXT("not_a_valid_type"), TEXT(""));
		TestFalse(TEXT("False on invalid type string"), bAdded);
	});

	It("AddViewModelProperty with duplicate name returns false", [this]()
	{
		FString ViewModelName = FString::Printf(TEXT("VM_Duplicate_Property_%d"), TestCounter++);
		UBlueprint* ViewModel = UMVVMToolset::CreateViewModel(ViewModelName, MVVMToolSetTest::TestMountPoint, nullptr);
		if (!IsValid(ViewModel))
		{
			AddInfo(TEXT("Skipped: [Invalid Test ViewModel Blueprint]"));
			return;
		}

		UMVVMToolset::AddViewModelProperty(ViewModel, TEXT("Health"), TEXT("float"), TEXT(""));
		bool bDuplicate = UMVVMToolset::AddViewModelProperty(ViewModel, TEXT("Health"), TEXT("float"), TEXT(""));
		TestFalse(TEXT("Duplicate property name rejected"), bDuplicate);
	});

	// ---- AddViewModelToWidget errors ----

	It("AddViewModelToWidget with null widget does not crash", [this]()
	{
		FString ViewModelName = FString::Printf(TEXT("VM_NullWidget_%d"), TestCounter++);
		UBlueprint* ViewModel = UMVVMToolset::CreateViewModel(ViewModelName, MVVMToolSetTest::TestMountPoint, nullptr);
		if (!IsValid(ViewModel))
		{
			AddInfo(TEXT("Skipped: [Invalid Test ViewModel Blueprint]"));
			return;
		}

		UMVVMToolset::AddViewModelToWidget(nullptr, ViewModel->SkeletonGeneratedClass);
		// No assert — just verifying no crash. ListWidgetViewModels(nullptr) handled separately.
		TestTrue(TEXT("No crash on null widget"), true);
	});

	It("AddViewModelToWidget with null class does not crash", [this]()
	{
		FString WidgetBlueprintName = FString::Printf(TEXT("WWidgetBlueprint_NullClass_%d"), TestCounter++);
		UWidgetBlueprint* LocalWidgetBlueprint = MVVMToolSetTest::CreateTestWidgetBlueprintWithMVVM(WidgetBlueprintName);
		TestNotNull(TEXT("WidgetBlueprint created"), LocalWidgetBlueprint);
		if (!IsValid(LocalWidgetBlueprint))
		{
			return;
		}

		UMVVMToolset::AddViewModelToWidget(LocalWidgetBlueprint, nullptr);
		TArray<UClass*> ViewModels = UMVVMToolset::ListWidgetViewModels(LocalWidgetBlueprint);
		TestEqual(TEXT("No ViewModel added with null class"), ViewModels.Num(), 0);
	});

	// ---- ListWidgetViewModels errors ----

	It("ListWidgetViewModels with null widget returns empty array", [this]()
	{
		TArray<UClass*> ViewModels = UMVVMToolset::ListWidgetViewModels(nullptr);
		TestEqual(TEXT("Empty on null widget"), ViewModels.Num(), 0);
	});

	// ---- ListWidgetViewBindings errors ----

	It("ListWidgetViewBindings with null widget returns empty array", [this]()
	{
		TArray<FMVVMBlueprintViewBinding> Bindings = UMVVMToolset::ListWidgetViewBindings(nullptr);
		TestEqual(TEXT("Empty on null widget"), Bindings.Num(), 0);
	});

	// ---- CreateViewBinding errors ----

	It("CreateViewBinding with null widget returns invalid GUID", [this]()
	{
		FGuid BindingId = UMVVMToolset::CreateViewBinding(nullptr, nullptr, TEXT("Health"), nullptr, TEXT("Health"), NAME_None);
		TestFalse(TEXT("Invalid GUID on null widget"), BindingId.IsValid());
	});

	It("CreateViewBinding with unresolvable property path returns invalid GUID", [this]()
	{
		FString WidgetBlueprintName = FString::Printf(TEXT("WWidgetBlueprint_BadPath_%d"), TestCounter++);
		UWidgetBlueprint* LocalWidgetBlueprint = MVVMToolSetTest::CreateTestWidgetBlueprintWithMVVM(WidgetBlueprintName);
		TestNotNull(TEXT("WidgetBlueprint created"), LocalWidgetBlueprint);
		if (!IsValid(LocalWidgetBlueprint))
		{
			return;
		}

		FGuid BindingId = UMVVMToolset::CreateViewBinding(
			LocalWidgetBlueprint,
			nullptr, TEXT("__NoSuchProperty"),
			nullptr, TEXT("__NoSuchProperty"),
			NAME_None);
		TestFalse(TEXT("Invalid GUID on unresolvable path"), BindingId.IsValid());
	});

	// ---- RemoveWidgetViewBinding errors ----

	It("RemoveWidgetViewBinding with bogus binding ID returns false", [this]()
	{
		FString WidgetBlueprintName = FString::Printf(TEXT("WWidgetBlueprint_RemoveErr_%d"), TestCounter++);
		UWidgetBlueprint* LocalWidgetBlueprint = MVVMToolSetTest::CreateTestWidgetBlueprintWithMVVM(WidgetBlueprintName);
		TestNotNull(TEXT("WidgetBlueprint created"), LocalWidgetBlueprint);
		if (!IsValid(LocalWidgetBlueprint))
		{
			return;
		}

		bool bRemoved = UMVVMToolset::RemoveWidgetViewBinding(LocalWidgetBlueprint, FGuid::NewGuid());
		TestFalse(TEXT("False on bogus binding ID"), bRemoved);
	});

	It("RemoveWidgetViewBinding with null widget returns false", [this]()
	{
		bool bRemoved = UMVVMToolset::RemoveWidgetViewBinding(nullptr, FGuid::NewGuid());
		TestFalse(TEXT("False on null widget"), bRemoved);
	});
}

// ============================================================================
// Registration Test
// ============================================================================

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FMVVMTest_Registration,
	"AI.Toolsets.MVVM.Registration.ToolsetRegistered",
	MVVMToolSetTest::Flags)

bool FMVVMTest_Registration::RunTest(const FString&)
{
	FString Schemas = UToolsetRegistry::GetAllToolsetJsonSchemas();
	TestTrue(TEXT("MVVMToolset in registry"), Schemas.Contains(TEXT("MVVMToolset")));
	return true;
}

#endif // WITH_DEV_AUTOMATION_TESTS
